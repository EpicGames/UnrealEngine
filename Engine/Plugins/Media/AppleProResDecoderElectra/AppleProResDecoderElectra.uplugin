{
  "FileVersion": 3,
  "Version": 1,
  "VersionName": "1.0",
  "FriendlyName": "Apple ProRes Decoder for Electra",
  "Description": "Implements video playback of Apple ProRes encoded videos. Apple ProRes is a high quality, lossy video compression format.",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "Category": "Media Players",
  "EnabledByDefault": false,
  "CanContainContent": false,
  "IsBetaVersion": false,
  "Installed": false,
  "SupportedTargetPlatforms": [ "Win64", "Mac" ],
  "Modules": [
    {
      "Name": "AppleProResDecoderElectra",
      "Type": "Runtime",
      "LoadingPhase": "PostEngineInit",
      "PlatformAllowList": [ "Win64", "Mac" ],
      "PlatformArchitectureDenyList": [ "Win64:arm64" ],
      "TargetDenyList": [ "Server" ]
    }
  ],
  "Plugins": [
    {
      "Name": "ElectraCodecs",
      "Enabled": true
    }
  ]
}
