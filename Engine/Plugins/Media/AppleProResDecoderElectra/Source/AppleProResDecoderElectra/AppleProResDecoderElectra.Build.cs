// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class AppleProResDecoderElectra : ModuleRules
	{
		public AppleProResDecoderElectra(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
                    "Projects",
					"ElectraBase",
                    "ElectraSamples",
					"ElectraCodecFactory",
					"ElectraDecoders",
                    "MP4Utilities"
                });

            if (Target.Platform == UnrealTargetPlatform.Win64)
            {
                PrivateDependencyModuleNames.AddRange(
                    new string[] {
                    "ProResLib",
                    });

				PublicDependencyModuleNames.Add("DirectX");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");

				if (Target.WindowsPlatform.Architecture != UnrealArch.Arm64)
				{
					PublicAdditionalLibraries.AddRange(new string[] {
						Path.Combine(Target.WindowsPlatform.DirectXLibDir, "dxerr.lib"),
					});
				}
			}
			else if (Target.Platform == UnrealTargetPlatform.Mac)
			{
				PublicFrameworks.AddRange(
					new string[] {
						"CoreMedia",
						"CoreVideo",
						"AVFoundation",
						"VideoToolbox",
						"QuartzCore"
					});
			}
		}
	}
}
