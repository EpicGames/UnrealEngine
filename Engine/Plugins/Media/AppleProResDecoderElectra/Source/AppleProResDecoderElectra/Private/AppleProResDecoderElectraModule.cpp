// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleProResDecoderElectraModule.h"
#include "ProResDecoder/ElectraMediaProResDecoder.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AppleProResDecoderElectraModule"

DEFINE_LOG_CATEGORY(LogProResElectraDecoder);

class FAppleProResElectraDecoderModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
		FElectraMediaProResDecoder::Startup();
	}

	void ShutdownModule() override
	{
		FElectraMediaProResDecoder::Shutdown();
	}

	bool SupportsDynamicReloading() override
	{
		// Codec could still be in use
		return false;
	}
};

IMPLEMENT_MODULE(FAppleProResElectraDecoderModule, AppleProResDecoderElectra);

#undef LOCTEXT_NAMESPACE
