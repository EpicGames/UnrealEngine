// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProResDecoder/ElectraMediaProResDecoder.h"
#include "AppleProResDecoderElectraModule.h"
#include "IElectraCodecRegistry.h"
#include "IElectraCodecFactory.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputVideo.h"
#include "ElectraDecodersUtils.h"
#include "MP4Utilities.h"	// to read the ProRes header, which is given in the form of an MP4 Atom
#include "Utils/ElectraBitstreamProcessorGenericVideo.h"
#include "AppleProResDecoderElectraModule.h"
#include "Apple/AppleElectraDecoderPlatformOutputFormatTypes.h"

#include <VideoToolbox/VideoToolbox.h>


#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_UNEXPECTED_PROBLEM					1
#define ERRCODE_INTERNAL_ALREADY_CLOSED						2
#define ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD				3
#define ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER			4
#define ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT				5
#define ERRCODE_INTERNAL_DID_NOT_GET_INPUT_BUFFER			6
#define ERRCODE_INTERNAL_DID_NOT_GET_OUTPUT_BUFFER			7
#define ERRCODE_INTERNAL_COULD_NOT_GET_OUTPUT_FORMAT		8
#define ERRCODE_INTERNAL_FAILED_TO_HANDLE_OUTPUT			9
#define ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE	10


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FVideoDecoderProResElectra;


class FVideoDecoderOutputProResElectra : public IElectraDecoderVideoOutput, public IElectraDecoderVideoOutputImageBuffers
{
public:
	virtual ~FVideoDecoderOutputProResElectra()
	{
		ReleaseOutputBuffer();
	}

	FTimespan GetPTS() const override
	{ return PTS; }
	uint64 GetUserValue() const override
	{ return UserValue; }

	EOutputType GetOutputType() const
	{ return EOutputType::Output; }
	int32 GetWidth() const override
	{ return Width - Crop.Left - Crop.Right; }
	int32 GetHeight() const override
	{ return Height - Crop.Top - Crop.Bottom; }
	int32 GetDecodedWidth() const override
	{ return Width; }
	int32 GetDecodedHeight() const override
	{ return Height; }
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{ return Crop; }
	int32 GetAspectRatioW() const override
	{ return AspectW; }
	int32 GetAspectRatioH() const override
	{ return AspectH; }
	int32 GetFrameRateNumerator() const override
	{ return FrameRateN; }
	int32 GetFrameRateDenominator() const override
	{ return FrameRateD; }
	int32 GetNumberOfBits() const override
	{ return NumBits; }
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{ OutExtraValues = ExtraValues; }
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		if (InTypeOfHandle == EElectraDecoderPlatformOutputHandleType::ImageBuffers)
		{
			return static_cast<IElectraDecoderVideoOutputImageBuffers*>(const_cast<FVideoDecoderOutputProResElectra*>(this));
		}
		return nullptr;
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{ return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported; }

	// Methods from IElectraDecoderVideoOutputImageBuffers
	uint32 GetCodec4CC() const override
	{
		return Codec4CC;
	}
	int32 GetNumberOfBuffers() const override
	{
		return 1;
	}
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const override
	{
		return nullptr;
	}
	void* GetBufferTextureByIndex(int32 InBufferIndex) const override
	{
		return InBufferIndex == 0 ? ImageBuffer : nullptr;
	}
	virtual bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const override
	{
		return false;
	}
	EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const override
	{
		return PixelFormat;
	}
	EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const override
	{
		return PixelEncoding;
	}
	int32 GetBufferPitchByIndex(int32 InBufferIndex) const override
	{
		return Pitch;
	}

public:
	void ReleaseOutputBuffer()
	{
		if (ImageBuffer)
		{
			CFRelease(ImageBuffer);
			ImageBuffer = nullptr;
		}
	}

	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 Width = 0;
	int32 Height = 0;
	int32 Pitch = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	EPixelFormat PixelFormat = EPixelFormat::PF_Unknown;
	EElectraTextureSamplePixelEncoding PixelEncoding = EElectraTextureSamplePixelEncoding::Native;
	TMap<FString, FVariant> ExtraValues;

	uint32 Codec4CC = 0;
	CVImageBufferRef ImageBuffer = nullptr;
};



class FVideoDecoderProResElectra : public IElectraDecoder
{
	static constexpr int32 kNumReorderFrames = 1;

public:

	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
	{
		OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)5));
		OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(false));
	}


	FVideoDecoderProResElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FVideoDecoderProResElectra();

	IElectraDecoder::EType GetType() const override
	{ return IElectraDecoder::EType::Video; }

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{
		TMap<FString, FVariant> DecoderFeatures;
		GetFeatures(DecoderFeatures);
		return FElectraDecoderBitstreamProcessorGenericVideo::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
	}

	void Suspend() override;
	void Resume() override;

private:
	static constexpr uint32 Make4CC(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
	{
		return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D);
	}

	struct FProResHeader
	{
		enum class EChromaType
		{
			Unknown,
			Chroma422,
			Chroma444,
		};
		enum class EStructureType
		{
			Unknown,
			Progressive,
			TopFieldFirst,
			BottomFieldFirst
		};
		enum class ESourcePixFormat
		{
			Unknown,
			Twovuy,		// 2vuy
			v210,
			v216,
			r408,
			v408,
			r4fl,
			RGB,
			BGRA,
			n302,
			b64a,
			R10k,
			l302,
			Invalid
		};
		enum class ESourceAlpha
		{
			None,
			A8,
			A16
		};
		void Reset()
		{
			FrameSizeInBytes = 0;
			FrameType = 0;
			HeaderSize = 0;
			Version = 0;
			CreatorID = 0;
			FrameWidth = 0;
			FrameHeight = 0;
			FrameFlags = 0;
			Reserved1 = 0;
			ColorPrimaries = 0;
			ColorTransferFunction = 0;
			ColorMatrix = 0;
			SourcePixFormat = 0;
		}
		EChromaType GetChromaType() const
		{
			switch(FrameFlags >> 6)
			{
				case 2: return EChromaType::Chroma422;
				case 3: return EChromaType::Chroma444;
				default: return EChromaType::Unknown;
			}
		}
		EStructureType GetStructureType() const
		{
			switch((FrameFlags >> 2) & 3)
			{
				case 0: return EStructureType::Progressive;
				case 1: return EStructureType::TopFieldFirst;
				case 2: return EStructureType::BottomFieldFirst;
				default: return EStructureType::Unknown;
			}
		}
		ESourcePixFormat GetSourcePixFormat() const
		{
			switch(SourcePixFormat >> 4)
			{
				case 0: return ESourcePixFormat::Unknown;
				case 1: return ESourcePixFormat::Twovuy;
				case 2: return ESourcePixFormat::v210;
				case 3: return ESourcePixFormat::v216;
				case 4: return ESourcePixFormat::r408;
				case 5: return ESourcePixFormat::v408;
				case 6: return ESourcePixFormat::r4fl;
				case 7: return ESourcePixFormat::RGB;
				case 8: return ESourcePixFormat::BGRA;
				case 9: return ESourcePixFormat::n302;
				case 10: return ESourcePixFormat::b64a;
				case 11: return ESourcePixFormat::R10k;
				case 12: return ESourcePixFormat::l302;
				default: return ESourcePixFormat::Invalid;
			}
		}
		ESourceAlpha GetSourceAlphaDepth() const
		{
			switch(SourcePixFormat & 15)
			{
				case 1: return ESourceAlpha::A8;
				case 2: return ESourceAlpha::A16;
				default: return ESourceAlpha::None;
			}
		}
		uint32 FrameSizeInBytes = 0;
		uint32 FrameType = 0;
		uint16 HeaderSize = 0;
		uint16 Version = 0;
		uint32 CreatorID = 0;
		uint16 FrameWidth = 0;
		uint16 FrameHeight = 0;
		uint8 FrameFlags = 0;
		uint8 Reserved1 = 0;
		uint8 ColorPrimaries = 0;
		uint8 ColorTransferFunction = 0;
		uint8 ColorMatrix = 0;
		uint8 SourcePixFormat = 0;
		// ... remainder is not of interest.
	};


	struct FDecoderHandle
	{
		FDecoderHandle() = default;

		~FDecoderHandle()
		{
			Close();
		}

		void Close()
		{
			if (DecompressionSession)
			{
				// Note: Do not finish delayed frames here. We don't want them and have to destroy the session immediately!
				//VTDecompressionSessionFinishDelayedFrames(DecompressionSession);
				VTDecompressionSessionWaitForAsynchronousFrames(DecompressionSession);
				VTDecompressionSessionInvalidate(DecompressionSession);
				CFRelease(DecompressionSession);
				DecompressionSession = nullptr;
			}
			if (FormatDescription)
			{
				CFRelease(FormatDescription);
				FormatDescription = nullptr;
			}
		}

		void Drain()
		{
			if (DecompressionSession)
			{
				// This waits until all pending frames have been processed.
				VTDecompressionSessionWaitForAsynchronousFrames(DecompressionSession);
			}
		}

		bool IsCompatibleWith(CMFormatDescriptionRef NewFormatDescription)
		{
			if (DecompressionSession)
			{
				Boolean bIsCompatible = VTDecompressionSessionCanAcceptFormatDescription(DecompressionSession, NewFormatDescription);
				return bIsCompatible;
			}
			return false;
		}
		CMFormatDescriptionRef		FormatDescription = nullptr;
		VTDecompressionSessionRef	DecompressionSession = nullptr;
	};

	struct FDecoderInput
	{
		FInputAccessUnit AccessUnit;
		TMap<FString, FVariant> AdditionalOptions;
	};

	struct FDecodedImage
	{
		FDecodedImage() = default;

		FDecodedImage(const FDecodedImage& rhs) : ImageBufferRef(nullptr)
		{
			InternalCopy(rhs);
		}

		FDecodedImage& operator=(const FDecodedImage& rhs)
		{
			if (this != &rhs)
			{
				InternalCopy(rhs);
			}
			return *this;
		}

		bool operator<(const FDecodedImage& rhs) const
		{
			return SourceInfo->AccessUnit.PTS < rhs.SourceInfo->AccessUnit.PTS;
		}

		void SetImageBufferRef(CVImageBufferRef InImageBufferRef)
		{
			if (ImageBufferRef)
			{
				CFRelease(ImageBufferRef);
				ImageBufferRef = nullptr;
			}
			if (InImageBufferRef)
			{
				CFRetain(InImageBufferRef);
				ImageBufferRef = InImageBufferRef;
			}
		}

		CVImageBufferRef ReleaseImageBufferRef()
		{
			CVImageBufferRef ref = ImageBufferRef;
			ImageBufferRef = nullptr;
			return ref;
		}

		~FDecodedImage()
		{
			if (ImageBufferRef)
			{
				CFRelease(ImageBufferRef);
			}
		}
		TSharedPtr<FDecoderInput, ESPMode::ThreadSafe>	SourceInfo;
	private:
		void InternalCopy(const FDecodedImage& rhs)
		{
			SourceInfo = rhs.SourceInfo;
			SetImageBufferRef(rhs.ImageBufferRef);
		}
		CVImageBufferRef ImageBufferRef = nullptr;
	};

	enum class EDecodeState
	{
		Decoding,
		Draining
	};

	void DecodeCallback(void* pSrcRef, OSStatus status, VTDecodeInfoFlags infoFlags, CVImageBufferRef imageBuffer, CMTime presentationTimeStamp, CMTime presentationDuration);
	static void _DecodeCallback(void* pUser, void* pSrcRef, OSStatus status, VTDecodeInfoFlags infoFlags, CVImageBufferRef imageBuffer, CMTime presentationTimeStamp, CMTime presentationDuration)
	{ static_cast<FVideoDecoderProResElectra*>(pUser)->DecodeCallback(pSrcRef, status, infoFlags, imageBuffer, presentationTimeStamp, presentationDuration); }

	bool PostError(int32_t ApiReturnValue, FString Message, int32 Code);

	bool ParseHeader(const void* InData, int32 InNumData);
	bool ConfigureOutputForInputFormat();
	bool CreateFormatDescriptionFromOptions(CMFormatDescriptionRef& OutFormatDescription, const TMap<FString, FVariant>& InOptions);

	bool InternalDecoderCreate(const TMap<FString, FVariant>& InAdditionalOptions);
	void InternalDecoderDestroy();
	void InternalFlushAllInputAndOutput();

	bool CheckForAvailableOutput();
	enum class EConvertResult
	{
		Success,
		Failure,
		GotEOS
	};
	EConvertResult ConvertDecoderOutput();

	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;

	IElectraDecoder::FError LastError;

	FCriticalSection InDecoderInputLock;
	TArray<TSharedPtr<FDecoderInput, ESPMode::ThreadSafe>> InDecoderInput;

	TSharedPtr<FVideoDecoderOutputProResElectra, ESPMode::ThreadSafe> CurrentOutput;

	TSharedPtr<FDecoderHandle, ESPMode::ThreadSafe> Decoder;
	FProResHeader CurrentHeader;

	CMVideoCodecType SourceVideoCodec = kCMVideoCodecType_AppleProRes422;
	TArray<int32> OutputPixelFormats;
	int32 DisplayWidth = 0;
	int32 DisplayHeight = 0;
	int32 DecodedWidth = 0;
	int32 DecodedHeight = 0;
	int32 AspectW = 0;
	int32 AspectH = 0;

	FCriticalSection ReadyImageMutex;
	TArray<FDecodedImage> ReadyImages;

	EDecodeState DecodeState = EDecodeState::Decoding;
	bool bNewDecoderRequired = false;
};


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FProResVideoDecoderElectraFactory : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation, public IElectraCodecModularFeature, public TSharedFromThis<FProResVideoDecoderElectraFactory, ESPMode::ThreadSafe>
{
public:
	static constexpr uint32 Make4CC(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
	{
		return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D);
	}

	virtual ~FProResVideoDecoderElectraFactory()
	{ }

	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override
	{
		OutCodecFactories.Add(AsShared());
	}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		FVideoDecoderProResElectra::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		FString Name;
		bool bIsSupported = true;
		switch(InCodecFormat.FourCC)
		{
			case Make4CC('a','p','c','h'):
			{
				Name = TEXT("Apple ProRes 422 High Quality");
				break;
			}
			case Make4CC('a','p','c','n'):
			{
				Name = TEXT("Apple ProRes 422 Standard Definition");
				break;
			}
			case Make4CC('a','p','c','s'):
			{
				Name = TEXT("Apple ProRes 422 LT");
				break;
			}
			case Make4CC('a','p','c','o'):
			{
				Name = TEXT("Apple ProRes 422 Proxy");
				break;
			}
			case Make4CC('a','p','4','h'):
			{
				Name = TEXT("Apple ProRes 4444");
				break;
			}
			case Make4CC('a','p','4','x'):
			{
				Name = TEXT("Apple ProRes 4444 XQ");
				break;
			}
			case Make4CC('a','p','r','n'):
			{
				Name = TEXT("Apple ProRes RAW");
				break;
			}
			case Make4CC('a','p','r','h'):
			{
				Name = TEXT("Apple ProRes RAW HQ");
				break;
			}
			default:
			{
				bIsSupported = false;
				break;
			}
		}
		if (bIsSupported)
		{
			if (InCodecFormat.HumanReadableFormatInfo.IsEmpty())
			{
				OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, FVariant(Name));
			}
			else
			{
				OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, InCodecFormat.HumanReadableFormatInfo);
			}
			OutFormatInfo.Emplace(IElectraDecoderFormatInfo::IsEveryFrameKeyframe, FVariant(true));
			return 1;
		}
		return 0;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? MakeShared<FVideoDecoderProResElectra, ESPMode::ThreadSafe>(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("Video Toolbox")); }
	FString GetVersion() const override
	{ return FString(); }
	FString GetImplementation() const override
	{ return FString(); }
	FString GetVendor() const override
	{ return(FString(TEXT("Apple"))); }

	static TSharedPtr<FProResVideoDecoderElectraFactory, ESPMode::ThreadSafe> Self;
	static TArray<FString> Permitted4CCs;
};
TSharedPtr<FProResVideoDecoderElectraFactory, ESPMode::ThreadSafe> FProResVideoDecoderElectraFactory::Self;
TArray<FString> FProResVideoDecoderElectraFactory::Permitted4CCs = { TEXT("apch"), TEXT("apcn"), TEXT("apcs"), TEXT("apco"), TEXT("ap4h"), TEXT("ap4x"), TEXT("aprn"), TEXT("aprh") };

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

void FElectraMediaProResDecoder::Startup()
{
	VTRegisterProfessionalVideoWorkflowVideoDecoders();

	// Make sure the codec factory module has been loaded.
	FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));

	// Create an instance of the factory, which is also the modular feature.
	check(!FProResVideoDecoderElectraFactory::Self.IsValid());
	FProResVideoDecoderElectraFactory::Self = MakeShared<FProResVideoDecoderElectraFactory, ESPMode::ThreadSafe>();
	// Register as modular feature.
	IModularFeatures::Get().RegisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FProResVideoDecoderElectraFactory::Self.Get());
}

void FElectraMediaProResDecoder::Shutdown()
{
	IModularFeatures::Get().UnregisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FProResVideoDecoderElectraFactory::Self.Get());
	FProResVideoDecoderElectraFactory::Self.Reset();
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

FVideoDecoderProResElectra::FVideoDecoderProResElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
	const Electra::FCodecTypeFormat::FVideo& vid(InitialCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
	AspectW = (int32) vid.AspectRatioW;
	AspectH = (int32) vid.AspectRatioH;
	DisplayWidth = (int32) vid.Width;
	DisplayHeight = (int32) vid.Height;
	// The decoded width is the same as the display width
	DecodedWidth = DisplayWidth;
	DecodedHeight = DisplayHeight;
}

FVideoDecoderProResElectra::~FVideoDecoderProResElectra()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

bool FVideoDecoderProResElectra::CreateFormatDescriptionFromOptions(CMFormatDescriptionRef& OutFormatDescription, const TMap<FString, FVariant>& InOptions)
{
	CFDictionaryRef NoExtras = nullptr;
	OSStatus Result = CMVideoFormatDescriptionCreate(kCFAllocatorDefault, SourceVideoCodec, DecodedWidth, DecodedHeight, NoExtras, &OutFormatDescription);
	if (Result == 0)
	{
		return true;
	}
	if (OutFormatDescription)
	{
		CFRelease(OutFormatDescription);
		OutFormatDescription = nullptr;
	}
	return PostError(Result, TEXT("Failed to create video format description"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
}


void FVideoDecoderProResElectra::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FVideoDecoderProResElectra::GetError() const
{
	return LastError;
}

void FVideoDecoderProResElectra::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

bool FVideoDecoderProResElectra::ResetToCleanStart()
{
	InternalDecoderDestroy();
	InternalFlushAllInputAndOutput();
	DecodeState = EDecodeState::Decoding;
	return !LastError.IsSet();
}

void FVideoDecoderProResElectra::InternalFlushAllInputAndOutput()
{
	InDecoderInputLock.Lock();
	InDecoderInput.Empty();
	InDecoderInputLock.Unlock();

	ReadyImageMutex.Lock();
	ReadyImages.Empty();
	ReadyImageMutex.Unlock();

	CurrentOutput.Reset();
}


IElectraDecoder::ECSDCompatibility FVideoDecoderProResElectra::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	// When we have no decoder yet then we are compatible because we will be creating a decoder when needed.
	if (!Decoder.IsValid() || !Decoder->DecompressionSession || !Decoder->FormatDescription)
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}
	return IElectraDecoder::ECSDCompatibility::DrainAndReset;
}

bool FVideoDecoderProResElectra::ParseHeader(const void* InData, int32 InNumData)
{
	if (InNumData < 36)
	{
		return false;
	}
	CurrentHeader.Reset();
	MP4Utilities::FMP4AtomReader Reader(MakeConstArrayView<const uint8>(reinterpret_cast<const uint8*>(InData), InNumData));
	Reader.Read(CurrentHeader.FrameSizeInBytes);
	Reader.Read(CurrentHeader.FrameType);
	if (CurrentHeader.FrameType != MP4Utilities::MakeBoxAtom('i','c','p','f'))
	{
		return false;
	}
	Reader.Read(CurrentHeader.HeaderSize);
	if (CurrentHeader.HeaderSize < 28)
	{
		return false;
	}
	Reader.Read(CurrentHeader.Version);
	Reader.Read(CurrentHeader.CreatorID);
	Reader.Read(CurrentHeader.FrameWidth);
	Reader.Read(CurrentHeader.FrameHeight);
	Reader.Read(CurrentHeader.FrameFlags);
	Reader.Read(CurrentHeader.Reserved1);
	Reader.Read(CurrentHeader.ColorPrimaries);
	Reader.Read(CurrentHeader.ColorTransferFunction);
	Reader.Read(CurrentHeader.ColorMatrix);
	Reader.Read(CurrentHeader.SourcePixFormat);
	return true;
}

bool FVideoDecoderProResElectra::ConfigureOutputForInputFormat()
{
	// Let Apple's decoder choose one of these formats depending on the input...
	OutputPixelFormats = {
		kCVPixelFormatType_4444AYpCbCr16,
		kCVPixelFormatType_420YpCbCr8BiPlanarVideoRange, kCVPixelFormatType_420YpCbCr8BiPlanarFullRange,
		kCVPixelFormatType_420YpCbCr10BiPlanarVideoRange, kCVPixelFormatType_420YpCbCr10BiPlanarFullRange
	};

	// Select the input format from the 4CC code
	switch(InitialCodecFormat.FourCC)
	{
		default:
		case Make4CC('a','p','c','h'):
		{
			SourceVideoCodec = kCMVideoCodecType_AppleProRes422HQ;
			break;
		}
		case Make4CC('a','p','c','n'):
		{
			SourceVideoCodec = kCMVideoCodecType_AppleProRes422;
			break;
		}
		case Make4CC('a','p','c','s'):
		{
			SourceVideoCodec = kCMVideoCodecType_AppleProRes422LT;
			break;
		}
		case Make4CC('a','p','c','o'):
		{
			SourceVideoCodec = kCMVideoCodecType_AppleProRes422Proxy;
			break;
		}
		case Make4CC('a','p','4','h'):
		{
			SourceVideoCodec = kCMVideoCodecType_AppleProRes4444;
			break;
		}
		case Make4CC('a','p','4','x'):
		{
			SourceVideoCodec = kCMVideoCodecType_AppleProRes4444XQ;
			break;
		}
		case Make4CC('a','p','r','n'):
		{
			SourceVideoCodec = kCMVideoCodecType_AppleProResRAW;
			break;
		}
		case Make4CC('a','p','r','h'):
		{
			SourceVideoCodec = kCMVideoCodecType_AppleProResRAWHQ;
			break;
		}
	}

	return true;
}


IElectraDecoder::EDecoderError FVideoDecoderProResElectra::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// CSD only buffer is not handled at the moment.
	check((InInputAccessUnit.Flags & EElectraDecoderFlags::InitCSDOnly) == EElectraDecoderFlags::None);

	// If a new decoder is needed, destroy the current one.
	if (bNewDecoderRequired)
	{
		InternalDecoderDestroy();
	}

	// Decode the data if given.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		if (!Decoder.IsValid())
		{
			if (!ParseHeader(InInputAccessUnit.Data, (int32) InInputAccessUnit.DataSize))
			{
				PostError(0, TEXT("Failed to parse the ProRes header"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
			// Is there a resolution mismatch for some reason?
			if (DecodedWidth != CurrentHeader.FrameWidth || DecodedHeight != CurrentHeader.FrameHeight)
			{
				DecodedWidth = CurrentHeader.FrameWidth;
				DecodedHeight = CurrentHeader.FrameHeight;
			}
			// Determine the best output format based on the current input.
			if (!ConfigureOutputForInputFormat())
			{
				PostError(0, TEXT("Failed to configure the ProRes output for the current input format"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
		}

		// Create decoder if necessary.
		if (!Decoder.IsValid() && !InternalDecoderCreate(InAdditionalOptions))
		{
			return IElectraDecoder::EDecoderError::Error;
		}

		CMBlockBufferRef AUDataBlock = nullptr;
		SIZE_T AUDataSize = (SIZE_T ) InInputAccessUnit.DataSize;
		OSStatus Result = CMBlockBufferCreateWithMemoryBlock(nullptr, nullptr, AUDataSize, nullptr, nullptr, 0, AUDataSize, kCMBlockBufferAssureMemoryNowFlag, &AUDataBlock);
		if (Result != kCMBlockBufferNoErr)
		{
			PostError(Result, TEXT("Failed to create decoder input memory block"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		Result = CMBlockBufferReplaceDataBytes(InInputAccessUnit.Data, AUDataBlock, 0, InInputAccessUnit.DataSize);
		if (Result != kCMBlockBufferNoErr)
		{
			CFRelease(AUDataBlock);
			PostError(Result, TEXT("Failed to setup decoder input memory block"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		// Set up the timing info with DTS, PTS and duration.
		CMSampleTimingInfo TimingInfo;
		const int64_t HNS_PER_S = 10000000;
		TimingInfo.decodeTimeStamp = CMTimeMake(InInputAccessUnit.DTS.GetTicks(), HNS_PER_S);
		TimingInfo.presentationTimeStamp = CMTimeMake(InInputAccessUnit.PTS.GetTicks(), HNS_PER_S);
		TimingInfo.duration = CMTimeMake(InInputAccessUnit.Duration.GetTicks(), HNS_PER_S);

		CMSampleBufferRef SampleBufferRef = nullptr;
		Result = CMSampleBufferCreate(kCFAllocatorDefault, AUDataBlock, true, nullptr, nullptr, Decoder->FormatDescription, 1, 1, &TimingInfo, 0, nullptr, &SampleBufferRef);
		// The buffer is now held by the sample, so we release our ref count.
		CFRelease(AUDataBlock);
		if (Result)	// see CMSampleBuffer for kCMSampleBufferError_AllocationFailed and such.
		{
			PostError(Result, TEXT("Failed to create video sample buffer"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		// Add to the list of inputs passed to the decoder.
		TSharedPtr<FDecoderInput, ESPMode::ThreadSafe> In(new FDecoderInput);
		In->AdditionalOptions = InAdditionalOptions;
		In->AccessUnit = InInputAccessUnit;
		// Zero the input pointer and size in the copy. That data is not owned by us and it's best not to have any
		// values here that would lead us to think that we do.
		In->AccessUnit.Data = nullptr;
		In->AccessUnit.DataSize = 0;
		InDecoderInputLock.Lock();
		InDecoderInput.Emplace(In);
		InDecoderInput.Sort([](const TSharedPtr<FDecoderInput, ESPMode::ThreadSafe>& e1, const TSharedPtr<FDecoderInput, ESPMode::ThreadSafe>& e2)
		{
			return e1->AccessUnit.PTS < e2->AccessUnit.PTS;
		});
		InDecoderInputLock.Unlock();

		// Decode
		VTDecodeFrameFlags DecodeFlags = kVTDecodeFrame_EnableAsynchronousDecompression | kVTDecodeFrame_EnableTemporalProcessing;
		if ((InInputAccessUnit.Flags & (/*EElectraDecoderFlags::DoNotOutput |*/ EElectraDecoderFlags::IsReplaySample | EElectraDecoderFlags::IsLastReplaySample)) != EElectraDecoderFlags::None)
		{
			DecodeFlags |= kVTDecodeFrame_DoNotOutputFrame;
		}
		VTDecodeInfoFlags  InfoFlags = 0;
		Result = VTDecompressionSessionDecodeFrame(Decoder->DecompressionSession, SampleBufferRef, DecodeFlags, In.Get(), &InfoFlags);
		CFRelease(SampleBufferRef);
		if (Result)
		{
			InDecoderInputLock.Lock();
			InDecoderInput.Remove(In);
			InDecoderInputLock.Unlock();
			// Did we lose the session?
			if (Result == kVTInvalidSessionErr)
			{
				InternalFlushAllInputAndOutput();
				bNewDecoderRequired = true;
				return IElectraDecoder::EDecoderError::LostDecoder;
			}

			PostError(Result, TEXT("Failed to decode video sample buffer"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderProResElectra::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Already draining?
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is a decoder send an end-of-stream and drain message.
	if (Decoder.IsValid())
	{
		DecodeState = EDecodeState::Draining;
		// Note: This call will block until all frames have been processed!
		Decoder->Drain();
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderProResElectra::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	InternalDecoderDestroy();
	InternalFlushAllInputAndOutput();
	DecodeState = EDecodeState::Decoding;
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FVideoDecoderProResElectra::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}

	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}

	if (bNewDecoderRequired || !Decoder.IsValid())
	{
		return IElectraDecoder::EOutputStatus::NeedInput;
	}

	// See if there is new output.
	if (CheckForAvailableOutput())
	{
		EConvertResult CnvRes = ConvertDecoderOutput();
		if (CnvRes == EConvertResult::Success)
		{
			return IElectraDecoder::EOutputStatus::Available;
		}
		else if (CnvRes == EConvertResult::GotEOS)
		{
			// Clear out the internal state for neatness.
			DecodeState = EDecodeState::Decoding;
			InternalFlushAllInputAndOutput();
			// For now we always create a new decoder. This could maybe be skipped
			// if the next access unit is compatible with the current configuration, but we're not sure
			// if discontinuities may be an issue so we don't risk this.
			bNewDecoderRequired = true;
			return IElectraDecoder::EOutputStatus::EndOfData;
		}
		else
		{
			return IElectraDecoder::EOutputStatus::Error;
		}
	}

	// With no output available, if we sent an EOS then we don't need new input.
	return DecodeState == EDecodeState::Draining ? IElectraDecoder::EOutputStatus::TryAgainLater : IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FVideoDecoderProResElectra::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}

void FVideoDecoderProResElectra::Suspend()
{
}

void FVideoDecoderProResElectra::Resume()
{
}

bool FVideoDecoderProResElectra::PostError(int32_t ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

bool FVideoDecoderProResElectra::InternalDecoderCreate(const TMap<FString, FVariant>& InAdditionalOptions)
{
	CMFormatDescriptionRef NewFormatDescr = nullptr;
	if (!CreateFormatDescriptionFromOptions(NewFormatDescr, InAdditionalOptions))
	{
		return false;
	}

	Decoder = MakeShared<FDecoderHandle, ESPMode::ThreadSafe>();;
	Decoder->FormatDescription = NewFormatDescr;

	VTDecompressionOutputCallbackRecord CallbackRecord;
	CallbackRecord.decompressionOutputCallback = _DecodeCallback;
	CallbackRecord.decompressionOutputRefCon   = this;

	// Output image format configuration
	CFMutableDictionaryRef OutputImageFormat = CFDictionaryCreateMutable(nullptr, 3, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);

	// Setup an array of suitable/preferred output formats.
	CFNumberRef* PixelFormatList = new CFNumberRef[OutputPixelFormats.Num()];
	for(int32 i=0; i<OutputPixelFormats.Num(); ++i)
	{
		PixelFormatList[i] = CFNumberCreate(kCFAllocatorDefault, kCFNumberSInt32Type, &OutputPixelFormats[i]);
	}
	CFArrayRef PixelFormatArray = CFArrayCreate(kCFAllocatorDefault, (const void**)PixelFormatList, OutputPixelFormats.Num(), &kCFTypeArrayCallBacks);
	CFDictionarySetValue(OutputImageFormat, kCVPixelBufferPixelFormatTypeKey, PixelFormatArray);
	// Choice of: kCVPixelBufferOpenGLCompatibilityKey (all)  kCVPixelBufferOpenGLESCompatibilityKey (iOS only)   kCVPixelBufferMetalCompatibilityKey (all)
	CFDictionarySetValue(OutputImageFormat, kCVPixelBufferMetalCompatibilityKey, kCFBooleanTrue);
#if 0 /// PLATFORM_IOS || PLATFORM_TVOS
	CFDictionarySetValue(OutputImageFormat, kCVPixelBufferOpenGLESCompatibilityKey, kCFBooleanFalse);
#endif
	CFRelease(PixelFormatArray);
	for(int32 i=0; i<OutputPixelFormats.Num(); ++i)
	{
		CFRelease(PixelFormatList[i]);
	}
	delete [] PixelFormatList;
	PixelFormatList = nullptr;

#if 1
	// Create session without configuration to use default hardware decoding.
	OSStatus Result = VTDecompressionSessionCreate(kCFAllocatorDefault, Decoder->FormatDescription, nullptr, OutputImageFormat, &CallbackRecord, &Decoder->DecompressionSession);
#else
	// Session configuration
	CFMutableDictionaryRef SessionConfiguration = CFDictionaryCreateMutable(nullptr, 1, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);
	OSStatus Result = VTDecompressionSessionCreate(kCFAllocatorDefault, Decoder->FormatDescription, SessionConfiguration, OutputImageFormat, &CallbackRecord, &Decoder->DecompressionSession);
	CFRelease(SessionConfiguration);
#endif

	CFRelease(OutputImageFormat);
	if (Result != 0)
	{
		PostError(Result, TEXT("VTDecompressionSessionCreate() failed to create video decoder"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
		return false;
	}

	InternalFlushAllInputAndOutput();
	bNewDecoderRequired = false;
	return true;
}

void FVideoDecoderProResElectra::InternalDecoderDestroy()
{
	if (Decoder.IsValid())
	{
		Decoder->Close();
		Decoder.Reset();
	}
	InternalFlushAllInputAndOutput();
}

bool FVideoDecoderProResElectra::CheckForAvailableOutput()
{
	// No decoder?
	if (!Decoder.IsValid())
	{
		return false;
	}

	FScopeLock lock(&ReadyImageMutex);
	bool bHaveEnough = ReadyImages.Num() >= kNumReorderFrames || DecodeState == EDecodeState::Draining;
	return bHaveEnough;
}

FVideoDecoderProResElectra::EConvertResult FVideoDecoderProResElectra::ConvertDecoderOutput()
{
	FScopeLock lock(&ReadyImageMutex);
	if (!ReadyImages.Num())
	{
		return DecodeState == EDecodeState::Draining ? EConvertResult::GotEOS : EConvertResult::Success;
	}
	FDecodedImage NextImage(ReadyImages[0]);
	ReadyImages.RemoveAt(0);
	lock.Unlock();

	TSharedPtr<FDecoderInput, ESPMode::ThreadSafe> In = NextImage.SourceInfo;

	TSharedPtr<FVideoDecoderOutputProResElectra, ESPMode::ThreadSafe> NewOutput = MakeShared<FVideoDecoderOutputProResElectra>();
	NewOutput->PTS = In->AccessUnit.PTS;
	NewOutput->UserValue = In->AccessUnit.UserValue;

	NewOutput->Width = DecodedWidth;
	NewOutput->Height = DecodedHeight;
	NewOutput->Pitch = NewOutput->Width;
	NewOutput->Crop.Right = DecodedWidth - DisplayWidth;
	NewOutput->Crop.Bottom = DecodedHeight - DisplayHeight;
	if (AspectW && AspectH)
	{
		NewOutput->AspectW = AspectW;
		NewOutput->AspectH = AspectH;
	}

	NewOutput->ImageBuffer = NextImage.ReleaseImageBufferRef();
	int pixelFormat = CVPixelBufferGetPixelFormatType(NewOutput->ImageBuffer);

	NewOutput->Codec4CC = InitialCodecFormat.FourCC;

	switch(pixelFormat)
	{
		case kCVPixelFormatType_4444AYpCbCr16:
		{
			NewOutput->PixelFormat = EPixelFormat::PF_A16B16G16R16;
			NewOutput->PixelEncoding = EElectraTextureSamplePixelEncoding::YCbCr_Alpha;
			break;
		}
		case kCVPixelFormatType_420YpCbCr8BiPlanarVideoRange:
		case kCVPixelFormatType_420YpCbCr8BiPlanarFullRange:
		{
			NewOutput->PixelFormat = EPixelFormat::PF_NV12;
			NewOutput->PixelEncoding = EElectraTextureSamplePixelEncoding::Native;
			break;
		}
		case kCVPixelFormatType_420YpCbCr10BiPlanarVideoRange:
		case kCVPixelFormatType_420YpCbCr10BiPlanarFullRange:
		{
			NewOutput->PixelFormat = EPixelFormat::PF_P010;
			NewOutput->PixelEncoding = EElectraTextureSamplePixelEncoding::Native;
			break;
		}
		default:
		{
			check(!"Unexpected output format!");
			NewOutput->PixelFormat = EPixelFormat::PF_Unknown;
			NewOutput->PixelEncoding = EElectraTextureSamplePixelEncoding::Native;
		}
	}

	NewOutput->NumBits = ElectraVideoDecoderFormatTypesApple::GetNumComponentBitsForPixelFormat(pixelFormat);
	if (NewOutput->NumBits < 0)
	{
		PostError(0, TEXT("Unrecognized CVPixelBuffer format (number of bits)"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}

	NewOutput->ExtraValues.Emplace(TEXT("platform"), FVariant(TEXT("apple")));
	NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("prores")));
	NewOutput->ExtraValues.Emplace(TEXT("codec_4cc"), FVariant(InitialCodecFormat.FourCC));

	CurrentOutput = MoveTemp(NewOutput);
	return EConvertResult::Success;
}


//-----------------------------------------------------------------------------
/**
 * Callback from the video decoder when a new decoded image is ready.
 *
 * @param pSrcRef
 * @param Result
 * @param infoFlags
 * @param imageBuffer
 * @param presentationTimeStamp
 * @param presentationDuration
 */
void FVideoDecoderProResElectra::DecodeCallback(void* pSrcRef, OSStatus Result, VTDecodeInfoFlags infoFlags, CVImageBufferRef imageBuffer, CMTime presentationTimeStamp, CMTime presentationDuration)
{
	// Remove the source info even if there ultimately was a decode error or if the frame was dropped.
	TSharedPtr<FDecoderInput, ESPMode::ThreadSafe> MatchingInput;
	InDecoderInputLock.Lock();
	int32 NumCurrentDecodeInputs = InDecoderInput.Num();
	for(int32 i=0; i<NumCurrentDecodeInputs; ++i)
	{
		if (InDecoderInput[i].Get() == pSrcRef)
		{
			MatchingInput = InDecoderInput[i];
			InDecoderInput.RemoveSingle(MatchingInput);
			break;
		}
	}
	InDecoderInputLock.Unlock();

	if (!MatchingInput.IsValid())
	{
		PostError(0, TEXT("There is no pending decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return;
	}

	if (Result == 0)
	{
		if (imageBuffer != nullptr && (infoFlags & kVTDecodeInfo_FrameDropped) == 0 && MatchingInput.IsValid())
		{
			FDecodedImage NextImage;
			NextImage.SourceInfo = MatchingInput;
			NextImage.SetImageBufferRef(imageBuffer);
			// Recall decoded frame for later processing. All output processing happens on the decoder thread.
 			ReadyImageMutex.Lock();
			ReadyImages.Add(NextImage);
			ReadyImages.Sort();
			ReadyImageMutex.Unlock();
		}
	}
	else
	{
		FString Msg = FString::Printf(TEXT("Failed to decode video: DecodeCallback() returned Result 0x%x; flags 0x%x"), Result, infoFlags);
		PostError(Result, Msg, ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
	}
}
