// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProResDecoder/ElectraMediaProResDecoder.h"
#include "AppleProResDecoderElectraModule.h"
#include "IElectraCodecRegistry.h"
#include "IElectraCodecFactory.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputVideo.h"
#include "ElectraDecodersUtils.h"
#include "MP4Utilities.h"	// to read the ProRes header, which is given in the form of an MP4 Atom
#include "Utils/ElectraBitstreamProcessorGenericVideo.h"

#include "ElectraDecodersPlatformResources.h"
#include COMPILED_PLATFORM_HEADER(ElectraTextureSampleGPUBufferHelper.h)

#include "ProResDecoder.h"

#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_ALREADY_CLOSED						1
#define ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT				2

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FVideoDecoderProResElectra;


class FVideoDecoderOutputProResElectra : public IElectraDecoderVideoOutput, public IElectraDecoderVideoOutputImageBuffers
{
public:
	virtual ~FVideoDecoderOutputProResElectra()
	{
	}

	FTimespan GetPTS() const override
	{
		return PTS;
	}
	uint64 GetUserValue() const override
	{
		return UserValue;
	}

	EOutputType GetOutputType() const
	{
		return EOutputType::Output;
	}

	int32 GetWidth() const override
	{
		return ImageWidth;
	}
	int32 GetHeight() const override
	{
		return ImageHeight;
	}
	int32 GetDecodedWidth() const override
	{
		return Width;
	}
	int32 GetDecodedHeight() const override
	{
		return Height;
	}
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{
		return Crop;
	}
	int32 GetAspectRatioW() const override
	{
		return AspectW;
	}
	int32 GetAspectRatioH() const override
	{
		return AspectH;
	}
	int32 GetFrameRateNumerator() const override
	{
		return FrameRateN;
	}
	int32 GetFrameRateDenominator() const override
	{
		return FrameRateD;
	}
	int32 GetNumberOfBits() const override
	{
		return NumBits;
	}
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{
		OutExtraValues = ExtraValues;
	}
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		if (InTypeOfHandle == EElectraDecoderPlatformOutputHandleType::ImageBuffers)
		{
			return static_cast<IElectraDecoderVideoOutputImageBuffers*>(const_cast<FVideoDecoderOutputProResElectra*>(this));
		}
		return nullptr;
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{
		return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported;
	}

	// Methods from IElectraDecoderVideoOutputImageBuffers
	uint32 GetCodec4CC() const override
	{
		return Codec4CC;
	}
	int32 GetNumberOfBuffers() const override
	{
		return 1;
	}
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const override
	{
		return InBufferIndex == 0 ? Buffer : nullptr;
	}
	void* GetBufferTextureByIndex(int32 InBufferIndex) const override
	{
#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (InBufferIndex == 0)
		{
			return GPUBuffer.Resource.GetReference();
		}
#endif
		return nullptr;
	}
	virtual bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const override
	{
#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (InBufferIndex == 0)
		{
			SyncObject = { GPUBuffer.Fence.GetReference(), GPUBuffer.FenceValue };
			return true;
		}
#endif
		return false;
	}
	EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const override
	{
		return BufferFormat;
	}
	EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const override
	{
		return BufferEncoding;
	}
	int32 GetBufferPitchByIndex(int32 InBufferIndex) const override
	{
		return Pitch;
	}

public:
	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 ImageWidth = 0;
	int32 ImageHeight = 0;
	int32 Width = 0;
	int32 Height = 0;
	int32 Pitch = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	int32 PixelFormat = 0;
	TMap<FString, FVariant> ExtraValues;

	uint32 Codec4CC = 0;
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> Buffer;
	EPixelFormat BufferFormat = EPixelFormat::PF_Unknown;
	EElectraTextureSamplePixelEncoding BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	FElectraMediaDecoderOutputBufferPool_DX12::FOutputData GPUBuffer;
#endif
};


class FVideoDecoderProResElectra : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
	{
		OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)5));
		OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(true));
	}

	FVideoDecoderProResElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FVideoDecoderProResElectra();

	IElectraDecoder::EType GetType() const override
	{
		return IElectraDecoder::EType::Video;
	}

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{
		TMap<FString, FVariant> DecoderFeatures;
		GetFeatures(DecoderFeatures);
		return FElectraDecoderBitstreamProcessorGenericVideo::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
	}

	void Suspend() override
	{ }
	void Resume() override
	{ }

private:
	struct FProResHeader
	{
		enum class EChromaType
		{
			Unknown,
			Chroma422,
			Chroma444,
		};
		enum class EStructureType
		{
			Unknown,
			Progressive,
			TopFieldFirst,
			BottomFieldFirst
		};
		enum class ESourcePixFormat
		{
			Unknown,
			Twovuy,		// 2vuy
			v210,
			v216,
			r408,
			v408,
			r4fl,
			RGB,
			BGRA,
			n302,
			b64a,
			R10k,
			l302,
			Invalid
		};
		enum class ESourceAlpha
		{
			None,
			A8,
			A16
		};
		void Reset()
		{
			FrameSizeInBytes = 0;
			FrameType = 0;
			HeaderSize = 0;
			Version = 0;
			CreatorID = 0;
			FrameWidth = 0;
			FrameHeight = 0;
			FrameFlags = 0;
			Reserved1 = 0;
			ColorPrimaries = 0;
			ColorTransferFunction = 0;
			ColorMatrix = 0;
			SourcePixFormat = 0;
		}
		EChromaType GetChromaType() const
		{
			switch(FrameFlags >> 6)
			{
				case 2: return EChromaType::Chroma422;
				case 3: return EChromaType::Chroma444;
				default: return EChromaType::Unknown;
			}
		}
		EStructureType GetStructureType() const
		{
			switch((FrameFlags >> 2) & 3)
			{
				case 0: return EStructureType::Progressive;
				case 1: return EStructureType::TopFieldFirst;
				case 2: return EStructureType::BottomFieldFirst;
				default: return EStructureType::Unknown;
			}
		}
		ESourcePixFormat GetSourcePixFormat() const
		{
			switch(SourcePixFormat >> 4)
			{
				case 0: return ESourcePixFormat::Unknown;
				case 1: return ESourcePixFormat::Twovuy;
				case 2: return ESourcePixFormat::v210;
				case 3: return ESourcePixFormat::v216;
				case 4: return ESourcePixFormat::r408;
				case 5: return ESourcePixFormat::v408;
				case 6: return ESourcePixFormat::r4fl;
				case 7: return ESourcePixFormat::RGB;
				case 8: return ESourcePixFormat::BGRA;
				case 9: return ESourcePixFormat::n302;
				case 10: return ESourcePixFormat::b64a;
				case 11: return ESourcePixFormat::R10k;
				case 12: return ESourcePixFormat::l302;
				default: return ESourcePixFormat::Invalid;
			}
		}
		ESourceAlpha GetSourceAlphaDepth() const
		{
			switch(SourcePixFormat & 15)
			{
				case 1: return ESourceAlpha::A8;
				case 2: return ESourceAlpha::A16;
				default: return ESourceAlpha::None;
			}
		}
		uint32 FrameSizeInBytes = 0;
		uint32 FrameType = 0;
		uint16 HeaderSize = 0;
		uint16 Version = 0;
		uint32 CreatorID = 0;
		uint16 FrameWidth = 0;
		uint16 FrameHeight = 0;
		uint8 FrameFlags = 0;
		uint8 Reserved1 = 0;
		uint8 ColorPrimaries = 0;
		uint8 ColorTransferFunction = 0;
		uint8 ColorMatrix = 0;
		uint8 SourcePixFormat = 0;
		// ... remainder is not of interest.
	};

	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);
	bool ParseHeader(const void* InData, int32 InNumData);
	bool ConfigureOutputForInputFormat();

	int32 DisplayWidth = 0;
	int32 DisplayHeight = 0;
	int32 DecodedWidth = 0;
	int32 DecodedHeight = 0;
	int32 AspectW = 0;
	int32 AspectH = 0;
	uint32 BufferAllocationSize = 0;

	IElectraDecoder::FError LastError;
	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;

	PRPixelFormat OutputPixelFormat = kPRFormat_y416;
	PRDownscaleMode OutDownscaleMode = kPRFullSize;
	int OutputBufferBytesPerRow = 0;
	bool OutputDiscardAlpha = false;
	TSharedPtr<FVideoDecoderOutputProResElectra, ESPMode::ThreadSafe> CurrentOutput;
	bool bFlushPending = false;

	PRDecoderRef Decoder = nullptr;
	FProResHeader CurrentHeader;

	uint32 MaxOutputBuffers;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	mutable TSharedPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> D3D12ResourcePool;
#endif
};


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FProResVideoDecoderElectraFactory : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation, public IElectraCodecModularFeature, public TSharedFromThis<FProResVideoDecoderElectraFactory, ESPMode::ThreadSafe>
{
public:
	static constexpr uint32 Make4CC(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
	{
		return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D);
	}

	virtual ~FProResVideoDecoderElectraFactory()
	{ }

	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override
	{
		OutCodecFactories.Add(AsShared());
	}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		FVideoDecoderProResElectra::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		FString Name;
		bool bIsSupported = true;
		switch(InCodecFormat.FourCC)
		{
			case Make4CC('a','p','c','h'):
			{
				Name = TEXT("Apple ProRes 422 High Quality");
				break;
			}
			case Make4CC('a','p','c','n'):
			{
				Name = TEXT("Apple ProRes 422 Standard Definition");
				break;
			}
			case Make4CC('a','p','c','s'):
			{
				Name = TEXT("Apple ProRes 422 LT");
				break;
			}
			case Make4CC('a','p','c','o'):
			{
				Name = TEXT("Apple ProRes 422 Proxy");
				break;
			}
			case Make4CC('a','p','4','h'):
			{
				Name = TEXT("Apple ProRes 4444");
				break;
			}
			case Make4CC('a','p','4','x'):
			{
				Name = TEXT("Apple ProRes 4444 XQ");
				break;
			}
			case Make4CC('a','p','r','n'):
			{
				Name = TEXT("Apple ProRes RAW");
				break;
			}
			case Make4CC('a','p','r','h'):
			{
				Name = TEXT("Apple ProRes RAW HQ");
				break;
			}
			default:
			{
				bIsSupported = false;
				break;
			}
		}
		if (bIsSupported)
		{
			if (InCodecFormat.HumanReadableFormatInfo.IsEmpty())
			{
				OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, FVariant(Name));
			}
			else
			{
				OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, InCodecFormat.HumanReadableFormatInfo);
			}
			OutFormatInfo.Emplace(IElectraDecoderFormatInfo::IsEveryFrameKeyframe, FVariant(true));
			return 1;
		}
		return 0;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? MakeShared<FVideoDecoderProResElectra, ESPMode::ThreadSafe>(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("ProResLib")); }
	FString GetVersion() const override
	{ return(FString()); }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString(TEXT("Apple"))); }

	static TSharedPtr<FProResVideoDecoderElectraFactory, ESPMode::ThreadSafe> Self;
};
TSharedPtr<FProResVideoDecoderElectraFactory, ESPMode::ThreadSafe> FProResVideoDecoderElectraFactory::Self;

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

void FElectraMediaProResDecoder::Startup()
{
	// Make sure the codec factory module has been loaded.
	FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));

	// Create an instance of the factory, which is also the modular feature.
	check(!FProResVideoDecoderElectraFactory::Self.IsValid());
	FProResVideoDecoderElectraFactory::Self = MakeShared<FProResVideoDecoderElectraFactory, ESPMode::ThreadSafe>();
	// Register as modular feature.
	IModularFeatures::Get().RegisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FProResVideoDecoderElectraFactory::Self.Get());
}

void FElectraMediaProResDecoder::Shutdown()
{
	IModularFeatures::Get().UnregisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FProResVideoDecoderElectraFactory::Self.Get());
	FProResVideoDecoderElectraFactory::Self.Reset();
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

FVideoDecoderProResElectra::FVideoDecoderProResElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
	const Electra::FCodecTypeFormat::FVideo& vid(InitialCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
	AspectW = (int32) vid.AspectRatioW;
	AspectH = (int32) vid.AspectRatioH;
	DisplayWidth = (int32) vid.Width;
	DisplayHeight = (int32) vid.Height;
	// The decoded width is the same as the display width
	DecodedWidth = DisplayWidth;
	DecodedHeight = DisplayHeight;

	MaxOutputBuffers = (uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_output_buffers"), 5);
	MaxOutputBuffers += kElectraDecoderPipelineExtraFrames;
}

FVideoDecoderProResElectra::~FVideoDecoderProResElectra()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

void FVideoDecoderProResElectra::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FVideoDecoderProResElectra::GetError() const
{
	return LastError;
}

bool FVideoDecoderProResElectra::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

void FVideoDecoderProResElectra::Close()
{
	ResetToCleanStart();
	if (Decoder)
	{
		PRCloseDecoder(Decoder);
		Decoder = nullptr;
	}
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

IElectraDecoder::ECSDCompatibility FVideoDecoderProResElectra::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	return IElectraDecoder::ECSDCompatibility::Compatible;
}

bool FVideoDecoderProResElectra::ResetToCleanStart()
{
	bFlushPending = false;
	CurrentOutput.Reset();
	return true;
}

bool FVideoDecoderProResElectra::ParseHeader(const void* InData, int32 InNumData)
{
	if (InNumData < 36)
	{
		return false;
	}
	CurrentHeader.Reset();
	MP4Utilities::FMP4AtomReader Reader(MakeConstArrayView<const uint8>(reinterpret_cast<const uint8*>(InData), InNumData));
	Reader.Read(CurrentHeader.FrameSizeInBytes);
	Reader.Read(CurrentHeader.FrameType);
	if (CurrentHeader.FrameType != MP4Utilities::MakeBoxAtom('i','c','p','f'))
	{
		return false;
	}
	Reader.Read(CurrentHeader.HeaderSize);
	if (CurrentHeader.HeaderSize < 28)
	{
		return false;
	}
	Reader.Read(CurrentHeader.Version);
	Reader.Read(CurrentHeader.CreatorID);
	Reader.Read(CurrentHeader.FrameWidth);
	Reader.Read(CurrentHeader.FrameHeight);
	Reader.Read(CurrentHeader.FrameFlags);
	Reader.Read(CurrentHeader.Reserved1);
	Reader.Read(CurrentHeader.ColorPrimaries);
	Reader.Read(CurrentHeader.ColorTransferFunction);
	Reader.Read(CurrentHeader.ColorMatrix);
	Reader.Read(CurrentHeader.SourcePixFormat);
	return true;
}

bool FVideoDecoderProResElectra::ConfigureOutputForInputFormat()
{
	switch(CurrentHeader.GetSourcePixFormat())
	{
		case FProResHeader::ESourcePixFormat::Twovuy:
		{
			OutputPixelFormat = kPRFormat_2vuy;
			break;
		}
		case FProResHeader::ESourcePixFormat::v210:
		{
			OutputPixelFormat = kPRFormat_v210;
			break;
		}
		case FProResHeader::ESourcePixFormat::v216:
		{
			OutputPixelFormat = kPRFormat_v216;
			break;
		}
		case FProResHeader::ESourcePixFormat::r4fl:
		{
			OutputPixelFormat = kPRFormat_r4fl;
			break;
		}
		case FProResHeader::ESourcePixFormat::b64a:
		{
			OutputPixelFormat = kPRFormat_b64a;
			break;
		}
		case FProResHeader::ESourcePixFormat::R10k:
		{
			OutputPixelFormat = kPRFormat_R10k;
			break;
		}
		case FProResHeader::ESourcePixFormat::r408:
		case FProResHeader::ESourcePixFormat::v408:
		case FProResHeader::ESourcePixFormat::RGB:
		case FProResHeader::ESourcePixFormat::BGRA:
		case FProResHeader::ESourcePixFormat::n302:
		case FProResHeader::ESourcePixFormat::l302:
		default:
		{
			OutputPixelFormat = kPRFormat_y416;
			break;
		}
	}

	//FProResHeader::EChromaType = CurrentHeader.GetChromaType();
	//FProResHeader::ESourceAlpha = CurrentHeader.GetSourceAlphaDepth();

	OutDownscaleMode = kPRFullSize;
	OutputDiscardAlpha = false;
	OutputBufferBytesPerRow = PRBytesPerRowNeededInPixelBuffer(DecodedWidth, OutputPixelFormat, OutDownscaleMode);
	BufferAllocationSize = OutputBufferBytesPerRow * DecodedHeight;
	return true;
}


IElectraDecoder::EDecoderError FVideoDecoderProResElectra::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

#if ELECTRA_MEDIAGPUBUFFER_DX12
	// If we will create a new resource pool or we have still buffers in an existing one, we can proceed, else we'd have no resources to output the data
	if (D3D12ResourcePool.IsValid() && !D3D12ResourcePool->BufferAvailable())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}
#endif

	// Create a decoder?
	if (!Decoder)
	{
		Decoder = PROpenDecoder(0, nullptr);
		if (!Decoder)
		{
			PostError(0, TEXT("PROpenDecoder() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
	}

	// Decode data. This immediately produces a new output frame.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		if (!ParseHeader(InInputAccessUnit.Data, (int32) InInputAccessUnit.DataSize))
		{
			PostError(0, TEXT("Failed to parse the ProRes header"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		// Is there a resolution mismatch for some reason?
		if (DecodedWidth != CurrentHeader.FrameWidth || DecodedHeight != CurrentHeader.FrameHeight)
		{
			DecodedWidth = CurrentHeader.FrameWidth;
			DecodedHeight = CurrentHeader.FrameHeight;
		}
		// Determine the best output format based on the current input.
		if (!ConfigureOutputForInputFormat())
		{
			PostError(0, TEXT("Failed to configure the ProRes output for the current input format"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		void* PlatformDevice = nullptr;
		int32 PlatformDeviceVersion = 0;
		bool bUseGPUBuffers = false;
#if ELECTRA_MEDIAGPUBUFFER_DX12
		FElectraDecodersPlatformResources::GetD3DDeviceAndVersion(&PlatformDevice, &PlatformDeviceVersion);
		bUseGPUBuffers = (PlatformDevice && PlatformDeviceVersion >= 12000);
#endif

		TSharedPtr<FVideoDecoderOutputProResElectra, ESPMode::ThreadSafe> NewOutput = MakeShared<FVideoDecoderOutputProResElectra>();
		NewOutput->PTS = InInputAccessUnit.PTS;
		NewOutput->UserValue = InInputAccessUnit.UserValue;

		NewOutput->Width = DecodedWidth;
		NewOutput->Height = DecodedHeight;
		NewOutput->Pitch = DecodedWidth;
		NewOutput->Crop.Right = DecodedWidth - DisplayWidth;
		NewOutput->Crop.Bottom = DecodedHeight - DisplayHeight;
		if (AspectW && AspectH)
		{
			NewOutput->AspectW = AspectW;
			NewOutput->AspectH = AspectH;
		}

		NewOutput->ImageWidth = DecodedWidth - NewOutput->Crop.Left - NewOutput->Crop.Right;
		NewOutput->ImageHeight = DecodedHeight - NewOutput->Crop.Top - NewOutput->Crop.Bottom;

		NewOutput->Codec4CC = InitialCodecFormat.FourCC;
		NewOutput->PixelFormat = static_cast<int32>(OutputPixelFormat);
		switch(OutputPixelFormat)
		{
			case kPRFormat_2vuy:
			{
				NewOutput->NumBits = 8;
				NewOutput->BufferFormat = EPixelFormat::PF_B8G8R8A8;
				NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::CbY0CrY1;
				NewOutput->Width /= 2;
				NewOutput->Pitch *= 2;
				break;
			}
			case kPRFormat_y416:
			{
				NewOutput->NumBits = 16;
				NewOutput->BufferFormat = EPixelFormat::PF_A16B16G16R16;
				NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::YCbCr_Alpha;
				NewOutput->Pitch *= 8;
				break;
			}
			case kPRFormat_r4fl:
			{
				NewOutput->NumBits = 32;
				NewOutput->BufferFormat = EPixelFormat::PF_A32B32G32R32F;
				NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::YCbCr_Alpha;
				NewOutput->Pitch *= 16;
				break;
			}
			case kPRFormat_v210:
			{
				NewOutput->NumBits = 10;
				NewOutput->BufferFormat = EPixelFormat::PF_A2B10G10R10;
				NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::CbY0CrY1;
				NewOutput->Width = 4 * ((NewOutput->Width + 5) / 6); // each 4 pixel contain 6 horizontally adjacent YCbCr pixels (incl. 4x 2-bit padding)
				NewOutput->Pitch = NewOutput->Width * 4;
				break;
			}
			case kPRFormat_v216:
			{
				NewOutput->NumBits = 16;
				NewOutput->BufferFormat = EPixelFormat::PF_A16B16G16R16;
				NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::CbY0CrY1;
				NewOutput->Width /= 2;
				NewOutput->Pitch *= 4;
				break;
			}
			case kPRFormat_b64a:
			{
				NewOutput->NumBits = 16;
				NewOutput->BufferFormat = EPixelFormat::PF_A16B16G16R16;
				NewOutput->BufferEncoding = EElectraTextureSamplePixelEncoding::ARGB_BigEndian;
				NewOutput->Pitch *= 8;
				break;
			}
			case kPRFormat_R10k:
			case kPRFormat_r210:
			default:
			{
				NewOutput->NumBits = 0;
				NewOutput->BufferFormat = EPixelFormat::PF_Unknown;
				break;
			}
		}

		NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("prores")));
		NewOutput->ExtraValues.Emplace(TEXT("codec_4cc"), FVariant(InitialCodecFormat.FourCC));

		PRPixelBuffer pb;
		FMemory::Memzero(pb);
		pb.format = OutputPixelFormat;
		pb.width = DecodedWidth;
		pb.height = DecodedHeight;

#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (bUseGPUBuffers)
		{
			// Setup a resource to directly receive the decoder output
			// (the memory is WC configured, so the decoder must not read from it -- this seems to be the case with ProRes)

			TRefCountPtr D3D12Device(static_cast<ID3D12Device*>(PlatformDevice));
			HRESULT Result;
			FString ResultMsg;

			// Create the resource pool as needed...
			if (!D3D12ResourcePool)
			{
				// note: prores will use a constant resolution and format throughout a decoder session
				D3D12ResourcePool = MakeShared<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe>(D3D12Device, MaxOutputBuffers, NewOutput->Width, NewOutput->Height, NewOutput->Pitch / NewOutput->Width);
			}

			// Request resource and fence...
			uint32 BufferPitch;
			D3D12ResourcePool->AllocateOutputDataAsBuffer(Result, ResultMsg, NewOutput->GPUBuffer, BufferPitch);
			if (NewOutput->GPUBuffer.Resource)
			{
				// Correct output pitch to what the resource is setup for
				NewOutput->Pitch = BufferPitch;

				void* BufferAddr;
				Result = NewOutput->GPUBuffer.Resource->Map(0, nullptr, &BufferAddr);
				if (SUCCEEDED(Result))
				{
					check((uint32)OutputBufferBytesPerRow <= BufferPitch);
					pb.baseAddr = (unsigned char*)BufferAddr;
					pb.rowBytes = BufferPitch;
				}
				else
				{
					PostError((int32)Result, TEXT("Failed to map DX12 texture buffer"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
					return IElectraDecoder::EDecoderError::Error;
				}
			}
			else
			{
				PostError((int32)Result, FString::Printf(TEXT("Failed to allocate DX12 texture buffer (%s)"), *ResultMsg), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
		else
#endif
		{
			NewOutput->Buffer = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
			NewOutput->Buffer->AddUninitialized(BufferAllocationSize);

			pb.baseAddr = NewOutput->Buffer->GetData();
			pb.rowBytes = OutputBufferBytesPerRow;
		}

		int NumBytesDecoded = PRDecodeFrame(Decoder, InInputAccessUnit.Data, static_cast<int>(InInputAccessUnit.DataSize), &pb, OutDownscaleMode, OutputDiscardAlpha);

#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (bUseGPUBuffers)
		{
			// Unmap the resource memory and signal that it's usable by the GPU
			NewOutput->GPUBuffer.Resource->Unmap(0, nullptr);

			if (NumBytesDecoded >= 0)
			{
				NewOutput->GPUBuffer.Fence->Signal(NewOutput->GPUBuffer.FenceValue);
			}
		}
#endif

		if (NumBytesDecoded < 0)
		{
			PostError(NumBytesDecoded, TEXT("PRDecodeFrame() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		CurrentOutput = MoveTemp(NewOutput);
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderProResElectra::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// Already draining?
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	bFlushPending = true;
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderProResElectra::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	ResetToCleanStart();
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FVideoDecoderProResElectra::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}
	// Have output?
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}
	// Pending flush?
	if (bFlushPending)
	{
		bFlushPending = false;
		return IElectraDecoder::EOutputStatus::EndOfData;
	}
	return IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FVideoDecoderProResElectra::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}
